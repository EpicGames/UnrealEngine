// Copyright Epic Games, Inc. All Rights Reserved.

#include "PLUGIN_NAME.h"
#include "PLUGIN_NAMEStyle.h"
#include "PLUGIN_NAMECommands.h"
#include "LevelEditor.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "ToolMenus.h"

static const FName PLUGIN_NAMETabName("PLUGIN_NAME");

#define LOCTEXT_NAMESPACE "FPLUGIN_NAMEModule"

void FPLUGIN_NAMEModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	
	FPLUGIN_NAMEStyle::Initialize();
	FPLUGIN_NAMEStyle::ReloadTextures();

	FPLUGIN_NAMECommands::Register();
	
	PluginCommands = MakeShareable(new FUICommandList);

	PluginCommands->MapAction(
		FPLUGIN_NAMECommands::Get().OpenPluginWindow,
		FExecuteAction::CreateRaw(this, &FPLUGIN_NAMEModule::PluginButtonClicked),
		FCanExecuteAction());

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FPLUGIN_NAMEModule::RegisterMenus));
	
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(PLUGIN_NAMETabName, FOnSpawnTab::CreateRaw(this, &FPLUGIN_NAMEModule::OnSpawnPluginTab))
		.SetDisplayName(LOCTEXT("FPLUGIN_NAMETabTitle", "PLUGIN_NAME"))
		.SetMenuType(ETabSpawnerMenuType::Hidden);
}

void FPLUGIN_NAMEModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	UToolMenus::UnRegisterStartupCallback(this);

	UToolMenus::UnregisterOwner(this);

	FPLUGIN_NAMEStyle::Shutdown();

	FPLUGIN_NAMECommands::Unregister();

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(PLUGIN_NAMETabName);
}

TSharedRef<SDockTab> FPLUGIN_NAMEModule::OnSpawnPluginTab(const FSpawnTabArgs& SpawnTabArgs)
{
	FText WidgetText = FText::Format(
		LOCTEXT("WindowWidgetText", "Add code to {0} in {1} to override this window's contents"),
		FText::FromString(TEXT("FPLUGIN_NAMEModule::OnSpawnPluginTab")),
		FText::FromString(TEXT("PLUGIN_NAME.cpp"))
		);

	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			// Put your tab content here!
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(WidgetText)
			]
		];
}

void FPLUGIN_NAMEModule::PluginButtonClicked()
{
	FGlobalTabmanager::Get()->TryInvokeTab(PLUGIN_NAMETabName);
}

void FPLUGIN_NAMEModule::RegisterMenus()
{
	// Owner will be used for cleanup in call to UToolMenus::UnregisterOwner
	FToolMenuOwnerScoped OwnerScoped(this);

	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Window");
		{
			FToolMenuSection& Section = Menu->FindOrAddSection("WindowLayout");
			Section.AddMenuEntryWithCommandList(FPLUGIN_NAMECommands::Get().OpenPluginWindow, PluginCommands);
		}
	}

	{
		UToolMenu* ToolbarMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.PlayToolBar");
		{
			FToolMenuSection& Section = ToolbarMenu->FindOrAddSection("PluginTools");
			{
				FToolMenuEntry& Entry = Section.AddEntry(FToolMenuEntry::InitToolBarButton(FPLUGIN_NAMECommands::Get().OpenPluginWindow));
				Entry.SetCommandList(PluginCommands);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FPLUGIN_NAMEModule, PLUGIN_NAME)