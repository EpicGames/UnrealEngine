// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

class FToolBarBuilder;
class FMenuBuilder;

class FPLUGIN_NAMEModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	/** This function will be bound to Command (by default it will bring up plugin window) */
	void PluginButtonClicked();
	
private:

	void RegisterMenus();

	TSharedRef<class SDockTab> OnSpawnPluginTab(const class FSpawnTabArgs& SpawnTabArgs);

private:
	TSharedPtr<class FUICommandList> PluginCommands;
};
