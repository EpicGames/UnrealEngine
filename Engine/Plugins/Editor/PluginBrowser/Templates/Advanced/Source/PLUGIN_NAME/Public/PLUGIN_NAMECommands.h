// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "PLUGIN_NAMEStyle.h"

class FPLUGIN_NAMECommands : public TCommands<FPLUGIN_NAMECommands>
{
public:

	FPLUGIN_NAMECommands()
		: TCommands<FPLUGIN_NAMECommands>(TEXT("PLUGIN_NAME"), NSLOCTEXT("Contexts", "PLUGIN_NAME", "PLUGIN_NAME Plugin"), NAME_None, FPLUGIN_NAMEStyle::GetStyleSetName())
	{
	}

	// TCommands<> interface
	virtual void RegisterCommands() override;

public:
	TSharedPtr< FUICommandInfo > OpenPluginWindow;
};